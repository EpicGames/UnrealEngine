{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "MovieSceneTextTrack",
	"Description": "Deprecated plugin. Text support moved to Movie Scene Tracks (built-in).",
	"Category": "Text",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "MovieSceneTextTrack",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "MovieSceneTextTrackEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}
