// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextComponentTypes.h"

namespace UE::MovieScene
{

PRAGMA_DISABLE_DEPRECATION_WARNINGS
static TUniquePtr<FTextComponentTypes> GTextComponentTypes;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

static bool GTextComponentTypesDestroyed = false;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FTextComponentTypes* FTextComponentTypes::Get()
{
	if (!GTextComponentTypes.IsValid())
	{
		check(!GTextComponentTypesDestroyed);
		GTextComponentTypes.Reset(new FTextComponentTypes);
	}
	return GTextComponentTypes.Get();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FTextComponentTypes::Destroy()
{
	GTextComponentTypes.Reset();
	GTextComponentTypesDestroyed = true;
}

}
