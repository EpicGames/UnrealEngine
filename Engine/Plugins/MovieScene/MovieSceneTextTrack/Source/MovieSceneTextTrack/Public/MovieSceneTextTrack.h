// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

/** Moved to MovieSceneTracks module */
UE_DEPRECATED_HEADER(5.7, "Use Tracks/MovieSceneTextTrack.h (MovieSceneTracks module) instead")
#include "Tracks/MovieSceneTextTrack.h"
