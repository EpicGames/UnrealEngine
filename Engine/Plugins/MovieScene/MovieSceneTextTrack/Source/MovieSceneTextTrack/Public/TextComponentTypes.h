// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/BuiltInComponentTypes.h"
#include "Misc/CoreMiscDefines.h"
#include "MovieSceneTracksComponentTypes.h"

namespace UE::MovieScene
{

struct UE_DEPRECATED(5.7, "FTextComponentTypes deprecated. Use FBuiltInComponentTypes and FMovieSceneTracksComponentTypes instead") FTextComponentTypes
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	MOVIESCENETEXTTRACK_API static FTextComponentTypes* Get();
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	MOVIESCENETEXTTRACK_API static void Destroy();

	// An FMovieSceneTextChannel
	TComponentTypeID<FSourceTextChannel> TextChannel;

	// Result of an evaluated FMovieSceneTextChannel
	TComponentTypeID<FText> TextResult;

	TPropertyComponents<FTextPropertyTraits> Text;

private:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FTextComponentTypes() = default;
	FTextComponentTypes(const FTextComponentTypes&) = default;
	FTextComponentTypes(FTextComponentTypes&&) = default;
	FTextComponentTypes& operator=(const FTextComponentTypes&) = default;
	FTextComponentTypes& operator=(FTextComponentTypes&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

}
