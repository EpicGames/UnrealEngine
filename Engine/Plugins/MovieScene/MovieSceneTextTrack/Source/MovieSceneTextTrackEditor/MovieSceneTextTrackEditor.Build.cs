// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MovieSceneTextTrackEditor : ModuleRules
{
	public MovieSceneTextTrackEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
		);
	}
}
